// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorLocatorEditor.h"
#include "Modules/ModuleManager.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocator.h"
#include "UniversalObjectLocatorEditor.h"
#include "IUniversalObjectLocatorEditorModule.h"
#include "IUniversalObjectLocatorCustomization.h"

#include "SceneOutlinerDragDrop.h"
#include "DragAndDrop/ActorDragDropOp.h"

#include "PropertyCustomizationHelpers.h"

#include "GameFramework/Actor.h"
#include "String/Split.h"
#include "UniversalObjectLocators/ActorLocatorFragment.h"


#define LOCTEXT_NAMESPACE "ActorLocatorEditor"

namespace UE::UniversalObjectLocator
{

ELocatorFragmentEditorType FActorLocatorEditor::GetLocatorFragmentEditorType() const
{
	return ELocatorFragmentEditorType::Absolute;
}

bool FActorLocatorEditor::IsDragSupported(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	TSharedPtr<FActorDragDropOp> ActorDrag;

	if (DragOperation->IsOfType<FSceneOutlinerDragDropOp>())
	{
		FSceneOutlinerDragDropOp* SceneOutlinerOp = static_cast<FSceneOutlinerDragDropOp*>(DragOperation.Get());
		ActorDrag = SceneOutlinerOp->GetSubOp<FActorDragDropOp>();
	}
	else if (DragOperation->IsOfType<FActorDragDropOp>())
	{
		ActorDrag = StaticCastSharedPtr<FActorDragDropOp>(DragOperation);
	}		

	if (ActorDrag)
	{
		for (const TWeakObjectPtr<AActor>& WeakActor : ActorDrag->Actors)
		{
			if (WeakActor.Get())
			{
				return true;
			}
		}
	}

	return false;
}

UObject* FActorLocatorEditor::ResolveDragOperation(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	TSharedPtr<FActorDragDropOp> ActorDrag;

	if (DragOperation->IsOfType<FSceneOutlinerDragDropOp>())
	{
		FSceneOutlinerDragDropOp* SceneOutlinerOp = static_cast<FSceneOutlinerDragDropOp*>(DragOperation.Get());
		ActorDrag = SceneOutlinerOp->GetSubOp<FActorDragDropOp>();
	}
	else if (DragOperation->IsOfType<FActorDragDropOp>())
	{
		ActorDrag = StaticCastSharedPtr<FActorDragDropOp>(DragOperation);
	}

	if (ActorDrag)
	{
		for (const TWeakObjectPtr<AActor>& WeakActor : ActorDrag->Actors)
		{
			if (AActor* Actor = WeakActor.Get())
			{
				return Actor;
			}
		}
	}

	return nullptr;
}

TSharedPtr<SWidget> FActorLocatorEditor::MakeEditUI(const FEditUIParameters& InParameters)
{
	AActor* InitialActor = GetActor(InParameters.Handle);
	const bool bAllowClear = true;
	const bool bAllowPickingLevelInstanceContent = true;
	const bool bDontDisplayUseSelected = false;
	const FOnActorSelected OnActorSelected = FOnActorSelected::CreateSP(this, &FActorLocatorEditor::OnSetActor, TWeakPtr<IFragmentEditorHandle>(InParameters.Handle));
	const FOnShouldFilterActor OnShouldFilterActor = FOnShouldFilterActor::CreateLambda([](const AActor*){ return true; });

	return
		SNew(SBox)
		.MinDesiredWidth(400.0f)
		.MaxDesiredWidth(400.0f)
		[
			PropertyCustomizationHelpers::MakeActorPickerWithMenu(
				InitialActor,
				bAllowClear,
				bAllowPickingLevelInstanceContent,
				OnShouldFilterActor,
				OnActorSelected,
				FSimpleDelegate(),
				FSimpleDelegate(),
				bDontDisplayUseSelected)
		];
}

AActor* FActorLocatorEditor::GetActor(TWeakPtr<IFragmentEditorHandle> InWeakHandle) const
{
	if (TSharedPtr<IFragmentEditorHandle> Handle = InWeakHandle.Pin())
	{
		const FUniversalObjectLocatorFragment& Fragment = Handle->GetFragment();
		ensure(Fragment.GetFragmentTypeHandle() == FActorLocatorFragment::FragmentType);
		const FActorLocatorFragment* Payload = Fragment.GetPayloadAs(FActorLocatorFragment::FragmentType);
		return (AActor*)Payload->Path.ResolveObject();
	}

	return nullptr;
}

FText FActorLocatorEditor::GetDisplayText(const FUniversalObjectLocatorFragment* InFragment) const
{
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FActorLocatorFragment::FragmentType);
		const FActorLocatorFragment* Payload = InFragment->GetPayloadAs(FActorLocatorFragment::FragmentType);
		if(Payload)
		{
			const FString& SubPathString = Payload->Path.GetSubPathString();
			if(!SubPathString.IsEmpty())
			{
				FStringView LevelStringView, ActorStringView;
				UE::String::SplitLast(SubPathString, TEXT("."), LevelStringView, ActorStringView);
				if(!ActorStringView.IsEmpty())
				{
					return FText::FromStringView(ActorStringView);
				}
			}
		}
	}

	return LOCTEXT("ExternalActorLocatorName", "Actor");
}

FText FActorLocatorEditor::GetDisplayTooltip(const FUniversalObjectLocatorFragment* InFragment) const
{
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FActorLocatorFragment::FragmentType);
		const FActorLocatorFragment* Payload = InFragment->GetPayloadAs(FActorLocatorFragment::FragmentType);
		if(Payload && Payload->Path.IsValid())
		{
			static const FTextFormat TextFormat(LOCTEXT("ExternalActorLocatorTooltipFormat", "A reference to actor {0}"));
			return FText::Format(TextFormat, FText::FromString(Payload->Path.ToString()));
		}
	}

	return LOCTEXT("ExternalActorLocatorTooltip", "An actor reference");
}

FSlateIcon FActorLocatorEditor::GetDisplayIcon(const FUniversalObjectLocatorFragment* InFragment) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Actor");
}

UClass* FActorLocatorEditor::ResolveClass(const FUniversalObjectLocatorFragment& InFragment, UObject* InContext) const
{
	if(UClass* Class = ILocatorFragmentEditor::ResolveClass(InFragment, InContext))
	{
		return Class;
	}

	return AActor::StaticClass();
}

void FActorLocatorEditor::OnSetActor(AActor* InActor, TWeakPtr<IFragmentEditorHandle> InWeakHandle)
{
	TSharedPtr<IFragmentEditorHandle> Handle = InWeakHandle.Pin();
	if (!Handle)
	{
		return;
	}

	FUniversalObjectLocatorFragment NewFragment(FActorLocatorFragment::FragmentType);
	FActorLocatorFragment* Payload = NewFragment.GetPayloadAs(FActorLocatorFragment::FragmentType);
	Payload->Path = InActor;
	Handle->SetValue(NewFragment);
}

FUniversalObjectLocatorFragment FActorLocatorEditor::MakeDefaultLocatorFragment() const
{
	FUniversalObjectLocatorFragment NewFragment(FActorLocatorFragment::FragmentType);
	return NewFragment;
}

} // namespace UE::UniversalObjectLocator

#undef LOCTEXT_NAMESPACE
