// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimInstanceLocatorEditor.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocators/AnimInstanceLocatorFragment.h"
#include "UniversalObjectLocator.h"
#include "UniversalObjectLocatorEditor.h"
#include "IUniversalObjectLocatorEditorModule.h"
#include "IUniversalObjectLocatorCustomization.h"
#include "ISequencerModule.h"
#include "SceneOutlinerDragDrop.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "PropertyCustomizationHelpers.h"
#include "Animation/AnimInstance.h"

#include "GameFramework/Actor.h"
#include "Components/SkeletalMeshComponent.h"


#define LOCTEXT_NAMESPACE "AnimInstanceLocatorEditor"

namespace UE::UniversalObjectLocator
{

class SAnimInstanceLocatorEditorUI : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SAnimInstanceLocatorEditorUI){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<IFragmentEditorHandle> InHandle)
	{
		WeakHandle = InHandle;

		const bool bCloseAfterSelection = true;
		FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr);

		FCanExecuteAction AlwaysExecute = FCanExecuteAction::CreateLambda([]{ return true; });

		MenuBuilder.AddMenuEntry(
			LOCTEXT("Menu_AnimInstanceLabel", "Anim Instance"),
			LOCTEXT("Menu_AnimInstanceTooltip", "Bind to the Anim Instance on the selected component"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SAnimInstanceLocatorEditorUI::ChangeType, EAnimInstanceLocatorFragmentType::AnimInstance),
				AlwaysExecute,
				FIsActionChecked::CreateSP(this, &SAnimInstanceLocatorEditorUI::CompareCurrentType, EAnimInstanceLocatorFragmentType::AnimInstance)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("Menu_PPAnimInstanceLabel", "Post Process Anim Instance"),
			LOCTEXT("Menu_PPAnimInstanceTooltip", "Bind to the Post Process Anim Instance on the selected component"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SAnimInstanceLocatorEditorUI::ChangeType, EAnimInstanceLocatorFragmentType::PostProcessAnimInstance),
				AlwaysExecute,
				FIsActionChecked::CreateSP(this, &SAnimInstanceLocatorEditorUI::CompareCurrentType, EAnimInstanceLocatorFragmentType::PostProcessAnimInstance)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		
		ChildSlot
		[
			MenuBuilder.MakeWidget()
		];
	}

private:
	FText GetCurrentAnimInstanceTypeText() const
	{
		TOptional<EAnimInstanceLocatorFragmentType> CommonType = GetCurrentType();
		if (!CommonType)
		{
			return LOCTEXT("AnimInstanceMixedLabel", "None");
		}

		if (CommonType.GetValue() == EAnimInstanceLocatorFragmentType::AnimInstance)
		{
			return LOCTEXT("AnimInstanceLabel", "Anim Instance");
		}
		return LOCTEXT("PostProcessAnimInstanceLabel", "Post Process Anim Instance");
	}

	void ChangeType(EAnimInstanceLocatorFragmentType InType)
	{
		TSharedPtr<IFragmentEditorHandle> Handle = WeakHandle.Pin();
		if (Handle)
		{
			FUniversalObjectLocatorFragment Fragment(FAnimInstanceLocatorFragment::FragmentType);
			FAnimInstanceLocatorFragment* Payload = Fragment.GetPayloadAs(FAnimInstanceLocatorFragment::FragmentType);
			Payload->Type = InType;
			Handle->SetValue(Fragment);
		}
	}

	bool CompareCurrentType(EAnimInstanceLocatorFragmentType InType) const
	{
		TOptional<EAnimInstanceLocatorFragmentType> Type = GetCurrentType();
		return Type.IsSet() && InType == Type.GetValue();
	}

	TOptional<EAnimInstanceLocatorFragmentType> GetCurrentType() const
	{
		TOptional<EAnimInstanceLocatorFragmentType> Type;

		TSharedPtr<IFragmentEditorHandle> Handle = WeakHandle.Pin();
		if (Handle)
		{
			const FUniversalObjectLocatorFragment& Fragment = Handle->GetFragment();
			const FAnimInstanceLocatorFragment* Payload = Fragment.GetPayloadAs(FAnimInstanceLocatorFragment::FragmentType);
			Type = Payload->Type;
		}

		return Type;
	}

	TWeakPtr<IFragmentEditorHandle> WeakHandle;
};

ELocatorFragmentEditorType FAnimInstanceLocatorEditor::GetLocatorFragmentEditorType() const
{
	return ELocatorFragmentEditorType::Relative;
}

bool FAnimInstanceLocatorEditor::IsDragSupported(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	TSharedPtr<FActorDragDropOp> ActorDrag;

	if (DragOperation->IsOfType<FSceneOutlinerDragDropOp>())
	{
		FSceneOutlinerDragDropOp* SceneOutlinerOp = static_cast<FSceneOutlinerDragDropOp*>(DragOperation.Get());
		ActorDrag = SceneOutlinerOp->GetSubOp<FActorDragDropOp>();
	}
	else if (DragOperation->IsOfType<FActorDragDropOp>())
	{
		ActorDrag = StaticCastSharedPtr<FActorDragDropOp>(DragOperation);
	}

	if (ActorDrag)
	{
		for (const TWeakObjectPtr<AActor>& WeakActor : ActorDrag->Actors)
		{
			if (WeakActor.Get())
			{
				return true;
			}
		}
	}

	return false;
}

UObject* FAnimInstanceLocatorEditor::ResolveDragOperation(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	TSharedPtr<FActorDragDropOp> ActorDrag;

	if (DragOperation->IsOfType<FSceneOutlinerDragDropOp>())
	{
		FSceneOutlinerDragDropOp* SceneOutlinerOp = static_cast<FSceneOutlinerDragDropOp*>(DragOperation.Get());
		ActorDrag = SceneOutlinerOp->GetSubOp<FActorDragDropOp>();
	}
	else if (DragOperation->IsOfType<FActorDragDropOp>())
	{
		ActorDrag = StaticCastSharedPtr<FActorDragDropOp>(DragOperation);
	}

	if (ActorDrag)
	{
		for (const TWeakObjectPtr<AActor>& WeakActor : ActorDrag->Actors)
		{
			if (AActor* Actor = WeakActor.Get())
			{
				return Actor;
			}
		}
	}

	return nullptr;
}

TSharedPtr<SWidget> FAnimInstanceLocatorEditor::MakeEditUI(const FEditUIParameters& InParameters)
{
	return SNew(SAnimInstanceLocatorEditorUI, InParameters.Handle);
}

FText FAnimInstanceLocatorEditor::GetDisplayText(const FUniversalObjectLocatorFragment* InFragment) const
{
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FAnimInstanceLocatorFragment::FragmentType);
		const FAnimInstanceLocatorFragment* AnimInstanceLocatorFragment = InFragment->GetPayloadAs(FAnimInstanceLocatorFragment::FragmentType);
		if(AnimInstanceLocatorFragment)
		{
			if(AnimInstanceLocatorFragment->Type == EAnimInstanceLocatorFragmentType::AnimInstance)
			{
				return LOCTEXT("AnimInstanceLocatorLabel", "Anim Instance");
			}
			else
			{
				return LOCTEXT("PostProcessInstanceLocatorLabel", "Post-process Anim Instance");
			}
		}
	}
	return LOCTEXT("AnimInstanceLocatorLabel", "Anim Instance");
}

FText FAnimInstanceLocatorEditor::GetDisplayTooltip(const FUniversalObjectLocatorFragment* InFragment) const
{
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FAnimInstanceLocatorFragment::FragmentType);
		const FAnimInstanceLocatorFragment* AnimInstanceLocatorFragment = InFragment->GetPayloadAs(FAnimInstanceLocatorFragment::FragmentType);
		if(AnimInstanceLocatorFragment)
		{
			if(AnimInstanceLocatorFragment->Type == EAnimInstanceLocatorFragmentType::AnimInstance)
			{
				return LOCTEXT("AnimInstanceLocatorTooltip", "A reference to an Anim Instance");
			}
			else
			{
				return LOCTEXT("PostProcessInstanceLocatorTooltip", "A reference to a Post-process Anim Instance");
			}
		}
	}

	return LOCTEXT("AnimInstanceLocatorTooltip", "A reference to an Anim Instance");
}

FSlateIcon FAnimInstanceLocatorEditor::GetDisplayIcon(const FUniversalObjectLocatorFragment* InFragment) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimBlueprint");
}

UClass* FAnimInstanceLocatorEditor::ResolveClass(const FUniversalObjectLocatorFragment& InFragment, UObject* InContext) const
{
	if(UClass* Class = ILocatorFragmentEditor::ResolveClass(InFragment, InContext))
	{
		return Class;
	}

	if(USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InContext))
	{
		if(SkeletalMeshComponent->AnimClass != nullptr)
		{
			return SkeletalMeshComponent->AnimClass;
		}
	}

	return UAnimInstance::StaticClass();
}

FUniversalObjectLocatorFragment FAnimInstanceLocatorEditor::MakeDefaultLocatorFragment() const
{
	FUniversalObjectLocatorFragment NewFragment(FAnimInstanceLocatorFragment::FragmentType);
	return NewFragment;
}

} // namespace UE::UniversalObjectLocator

#undef LOCTEXT_NAMESPACE
