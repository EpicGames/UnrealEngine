// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetLocatorEditor.h"
#include "Modules/ModuleManager.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocator.h"
#include "UniversalObjectLocatorEditor.h"
#include "IUniversalObjectLocatorEditorModule.h"
#include "IUniversalObjectLocatorCustomization.h"

#include "DragAndDrop/AssetDragDropOp.h"

#include "PropertyCustomizationHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Styling/SlateIconFinder.h"
#include "UniversalObjectLocators/AssetLocatorFragment.h"

#include "Widgets/Layout/SBox.h"


#define LOCTEXT_NAMESPACE "AssetLocatorEditor"

namespace UE::UniversalObjectLocator
{

ELocatorFragmentEditorType FAssetLocatorEditor::GetLocatorFragmentEditorType() const
{
	return ELocatorFragmentEditorType::Absolute;
}

bool FAssetLocatorEditor::IsDragSupported(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	TSharedPtr<FAssetDragDropOp> ActorDrag;

	if (DragOperation->IsOfType<FAssetDragDropOp>())
	{
		TSharedPtr<FAssetDragDropOp> AssetDrag = StaticCastSharedPtr<FAssetDragDropOp>(DragOperation);
		if (AssetDrag->GetAssets().Num() == 1)
		{
			return true;
		}
	}

	return false;
}

UObject* FAssetLocatorEditor::ResolveDragOperation(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	TSharedPtr<FAssetDragDropOp> ActorDrag;

	if (DragOperation->IsOfType<FAssetDragDropOp>())
	{
		TSharedPtr<FAssetDragDropOp> AssetDrag = StaticCastSharedPtr<FAssetDragDropOp>(DragOperation);
		if (AssetDrag->GetAssets().Num() == 1)
		{
			return AssetDrag->GetAssets()[0].FastGetAsset(true);
		}
	}

	return nullptr;
}

TSharedPtr<SWidget> FAssetLocatorEditor::MakeEditUI(const FEditUIParameters& InParameters)
{
	FAssetData InitialAsset = GetAsset(InParameters.Handle);
	const bool bAllowClear = true;
	const bool bAllowCopyPaste = true;
	const TArray<const UClass*> AllowedClasses = { UObject::StaticClass() };
	const FOnShouldFilterAsset OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([](const FAssetData&){ return false; });
	const FOnAssetSelected OnAssetSelected = FOnAssetSelected::CreateSP(this, &FAssetLocatorEditor::OnSetAsset, TWeakPtr<IFragmentEditorHandle>(InParameters.Handle));

	return
		SNew(SBox)
		.MinDesiredWidth(400.0f)
		.MaxDesiredWidth(400.0f)
		[
			PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
				InitialAsset,
				bAllowClear,
				bAllowCopyPaste,
				AllowedClasses,
				TArray<const UClass*>(),
				TArray<UFactory*>(),
				OnShouldFilterAsset,
				OnAssetSelected,
				FSimpleDelegate(),
				nullptr,
				TArray<FAssetData>())
		];
}

FAssetData FAssetLocatorEditor::GetAsset(TWeakPtr<IFragmentEditorHandle> InWeakHandle) const
{
	if (TSharedPtr<IFragmentEditorHandle> Handle = InWeakHandle.Pin())
	{
		const FUniversalObjectLocatorFragment& Fragment = Handle->GetFragment();
		ensure(Fragment.GetFragmentTypeHandle() == FAssetLocatorFragment::FragmentType);
		const FAssetLocatorFragment* Payload = Fragment.GetPayloadAs(FAssetLocatorFragment::FragmentType);
		const IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
		return AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(Payload->Path));
	}

	return FAssetData();
}

FText FAssetLocatorEditor::GetDisplayText(const FUniversalObjectLocatorFragment* InFragment) const
{
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FAssetLocatorFragment::FragmentType);
		const FAssetLocatorFragment* Payload = InFragment->GetPayloadAs(FAssetLocatorFragment::FragmentType);
		if(Payload)
		{
			return FText::FromName(Payload->Path.GetAssetName());
		}
	}

	return LOCTEXT("AssetLocatorName", "Asset");
}

FText FAssetLocatorEditor::GetDisplayTooltip(const FUniversalObjectLocatorFragment* InFragment) const
{
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FAssetLocatorFragment::FragmentType);
		const FAssetLocatorFragment* Payload = InFragment->GetPayloadAs(FAssetLocatorFragment::FragmentType);
		if(Payload)
		{
			static const FTextFormat TextFormat(LOCTEXT("AssetLocatorTooltipFormat", "A reference to asset {0}"));
			return FText::Format(TextFormat, FText::FromString(Payload->Path.ToString()));
		}
	}

	return LOCTEXT("AssetLocatorTooltip", "An asset reference");
}

FSlateIcon FAssetLocatorEditor::GetDisplayIcon(const FUniversalObjectLocatorFragment* InFragment) const
{
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FAssetLocatorFragment::FragmentType);
		const FAssetLocatorFragment* Payload = InFragment->GetPayloadAs(FAssetLocatorFragment::FragmentType);
		if(Payload)
		{
			IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
			FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(Payload->Path));
			return FSlateIconFinder::FindIconForClass(AssetData.GetClass());
		}
	}

	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Object");
}

UClass* FAssetLocatorEditor::ResolveClass(const FUniversalObjectLocatorFragment& InFragment, UObject* InContext) const
{
	if(UClass* Class = ILocatorFragmentEditor::ResolveClass(InFragment, InContext))
	{
		return Class;
	}

	return UObject::StaticClass();
}

void FAssetLocatorEditor::OnSetAsset(const FAssetData& InNewAsset, TWeakPtr<IFragmentEditorHandle> InWeakHandle)
{
	if (TSharedPtr<IFragmentEditorHandle> Handle = InWeakHandle.Pin())
	{
		// Assets are always absolute
		UObject* Object = InNewAsset.FastGetAsset(true);

		FUniversalObjectLocatorFragment NewFragment(FAssetLocatorFragment::FragmentType);
		FAssetLocatorFragment* Payload = NewFragment.GetPayloadAs(FAssetLocatorFragment::FragmentType);
		Payload->Path = FTopLevelAssetPath(Object);
		Handle->SetValue(NewFragment);
	}
}

FUniversalObjectLocatorFragment FAssetLocatorEditor::MakeDefaultLocatorFragment() const
{
	FUniversalObjectLocatorFragment NewFragment(FAssetLocatorFragment::FragmentType);
	return NewFragment;
}

} // namespace UE::UniversalObjectLocator

#undef LOCTEXT_NAMESPACE
