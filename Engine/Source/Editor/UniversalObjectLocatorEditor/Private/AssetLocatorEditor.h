// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UniversalObjectLocatorEditor.h"

struct FAssetData;

namespace UE::UniversalObjectLocator
{

class IUniversalObjectLocatorCustomization;

class FAssetLocatorEditor : public ILocatorFragmentEditor
{
	ELocatorFragmentEditorType GetLocatorFragmentEditorType() const override;

	bool IsDragSupported(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const override;

	UObject* ResolveDragOperation(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const override;

	TSharedPtr<SWidget> MakeEditUI(const FEditUIParameters& InParameters) override;

	FText GetDisplayText(const FUniversalObjectLocatorFragment* InFragment = nullptr) const override;

	FText GetDisplayTooltip(const FUniversalObjectLocatorFragment* InFragment = nullptr) const override;

	FSlateIcon GetDisplayIcon(const FUniversalObjectLocatorFragment* InFragment = nullptr) const override;

	UClass* ResolveClass(const FUniversalObjectLocatorFragment& InFragment, UObject* InContext) const override;

	FUniversalObjectLocatorFragment MakeDefaultLocatorFragment() const override;

private:
	FAssetData GetAsset(TWeakPtr<IFragmentEditorHandle> InWeakHandle) const;

	void OnSetAsset(const FAssetData& InNewObject, TWeakPtr<IFragmentEditorHandle> InWeakHandle);
};

} // namespace UE::UniversalObjectLocator

