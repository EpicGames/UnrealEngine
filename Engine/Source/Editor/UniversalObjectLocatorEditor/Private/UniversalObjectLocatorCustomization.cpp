// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocatorCustomization.h"
#include "UniversalObjectLocator.h"
#include "UniversalObjectLocatorFragmentType.h"
#include "UniversalObjectLocatorEditor.h"
#include "UniversalObjectLocatorEditorModule.h"

#include "Containers/Array.h"
#include "Internationalization/Internationalization.h"
#include "Modules/ModuleManager.h"

#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "PropertyCustomizationHelpers.h"

#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SDropTarget.h"

#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerModule.h"
#include "ScopedTransaction.h"
#include "UniversalObjectLocatorEditorContext.h"
#include "UniversalObjectLocatorInitializeResult.h"

#include "Async/Async.h"
#include "String/ParseTokens.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Views/STileView.h"
#include "IDetailChildrenBuilder.h"
#include "DetailLayoutBuilder.h"


#define LOCTEXT_NAMESPACE "FUniversalObjectLocatorCustomization"

namespace UE::UniversalObjectLocator
{

// Data representation of a UOL fragment for UI display
struct FFragmentItem : public IFragmentEditorHandle
{
public:
	// IFragmentEditorHandle interface
	virtual const FUniversalObjectLocatorFragment& GetFragment() const override
	{
		const FUniversalObjectLocator* Value = nullptr;
		PropertyHandle->EnumerateConstRawData(
			[&Value](const void* Data, int32 DataIndex, int32 Num)
			{
				Value = static_cast<const FUniversalObjectLocator*>(Data);
				return false;
			}
		);
		check(Value->Fragments.IsValidIndex(FragmentIndex));
		return Value->Fragments[FragmentIndex];
	}

	virtual const UClass* GetContextClass() const override
	{
		return WeakContextClass.Get();
	}

	virtual const UClass* GetResolvedClass() const override
	{
		return WeakResolvedClass.Get();
	}
	
	virtual void SetValue(const FUniversalObjectLocatorFragment& InNewValue) override
	{
		FScopedTransaction Transaction(LOCTEXT("EditLocatorFragmentTransaction", "Edit Locator Fragment"));

		PropertyHandle->NotifyPreChange();

		PropertyHandle->EnumerateRawData([this, &InNewValue](void* RawData, int32 Index, int32 Num){
			FUniversalObjectLocator* Reference = static_cast<FUniversalObjectLocator*>(RawData);
			check(Reference->Fragments.IsValidIndex(FragmentIndex));
			Reference->Fragments[FragmentIndex] = InNewValue;
			return true;
		});

		if(TSharedPtr<FUniversalObjectLocatorCustomization> Customization = WeakCustomization.Pin())
		{
			Customization->TrimAbsoluteFragments();
		}

		PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyHandle->NotifyFinishedChangingProperties();
	}

	// Fragment index in the overall fragment chain
	int32 FragmentIndex = INDEX_NONE;
	// The type of the fragment
	FFragmentTypeHandle FragmentType;
	// Locator editor type
	FName LocatorEditorType;
	// Locator editor
	TSharedPtr<ILocatorFragmentEditor> LocatorEditor;
	// Context class, if any
	TWeakObjectPtr<const UClass> WeakContextClass;
	// Resolved class, if any
	TWeakObjectPtr<const UClass> WeakResolvedClass;
	// Cached display text
	TOptional<FText> DisplayText;
	// Cached display tooltip text
	TOptional<FText> TooltipText;
	// Cached display icon
	TOptional<const FSlateBrush*> Icon;
	// Property handle to the containing FUniversalObjectLocator
	TSharedPtr<IPropertyHandle> PropertyHandle;
	// Owning customization
	TWeakPtr<FUniversalObjectLocatorCustomization> WeakCustomization; 
	// Whether this is the last fragment
	bool bIsTail = false;
};

TSharedRef<IPropertyTypeCustomization> FUniversalObjectLocatorCustomization::MakeInstance()
{
	return MakeShared<FUniversalObjectLocatorCustomization>();
}

void FUniversalObjectLocatorCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyHandle = StructPropertyHandle;
	PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();

	FSceneOutlinerInitializationOptions InitOptions;
	{
		InitOptions.bShowHeaderRow = false;
	}

	FUniversalObjectLocatorEditorModule& EditorModule = FModuleManager::Get().LoadModuleChecked<FUniversalObjectLocatorEditorModule>("UniversalObjectLocatorEditor");

	if (PropertyHandle->HasMetaData("LocatorContext"))
	{
		const FString& EditorContext = PropertyHandle->GetMetaData("LocatorContext");
		FName EditorContextName(*EditorContext);
		if(const TSharedPtr<ILocatorFragmentEditorContext> FoundContext = EditorModule.LocatorEditorContexts.FindRef(EditorContextName))
		{
			if(WeakContext.Get() == nullptr)
			{
				WeakContext = FoundContext->GetContext(PropertyHandle.ToSharedRef().Get());
				if(UObject* Context = WeakContext.Get())
				{
					if(Context->IsA<UClass>())
					{
						WeakContextClass = Cast<UClass>(Context);
					}
					else
					{
						WeakContextClass = Context->GetClass();
					}
				}
			}

			for (TPair<FName, TSharedPtr<ILocatorFragmentEditor>> Pair : EditorModule.LocatorEditors)
			{
				if(Pair.Value->IsAllowedInContext(EditorContextName) && FoundContext->IsFragmentAllowed(Pair.Key))
				{
					ApplicableLocators.Add(Pair.Key, Pair.Value);
				}
			}
		}
	}
	else
	{
		TArray<FString> AllowedTypes;
		if (PropertyHandle->HasMetaData("AllowedLocators"))
		{
			const FString& AllowedLocatorsNames = PropertyHandle->GetMetaData("AllowedLocators");
			UE::String::ParseTokens(AllowedLocatorsNames, TEXT(','), [&AllowedTypes](FStringView InToken)
			{
				AllowedTypes.Add(FString(InToken));
			}, UE::String::EParseTokensOptions::SkipEmpty | UE::String::EParseTokensOptions::Trim);
		}

		for (TPair<FName, TSharedPtr<ILocatorFragmentEditor>> Pair : EditorModule.LocatorEditors)
		{
			if ((AllowedTypes.Num() == 0 || AllowedTypes.Contains(Pair.Key.ToString())) && Pair.Value->IsAllowedInContext(NAME_None))
			{
				ApplicableLocators.Add(Pair.Key, Pair.Value);
			}
		}
		
		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);
		WeakContext = OuterObjects.Num() ? OuterObjects[0] : nullptr;
		WeakContextClass = OuterObjects.Num() ? OuterObjects[0]->GetClass() : UObject::StaticClass();
	}

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(400.0f)
	[
		SAssignNew(RootWidget, SBorder)
		.ToolTipText_Lambda([this]()
		{
			FTextBuilder TextBuilder;
			TextBuilder.AppendLine(PropertyHandle->GetProperty()->GetDisplayNameText());

			if(GetCachedData().PropertyValue.IsSet())
			{
				TStringBuilder<256> StringBuilder;
				GetCachedData().PropertyValue.GetValue().ToString(StringBuilder);
				TextBuilder.AppendLine(FText::FromStringView(StringBuilder));
			}
			else
			{
				TArray<UObject*> Objects;
				PropertyHandle->GetOuterObjects(Objects);
				if(Objects.Num() > 1)
				{
					TextBuilder.AppendLine(LOCTEXT("MultipleValues", "Multiple Values"));
				}
				else
				{
					TextBuilder.AppendLine(LOCTEXT("NoValues", "None"));
				}
			}

			return TextBuilder.ToText();
		})
		.BorderImage(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton").ButtonStyle.Normal)
		.Padding(2.0f, 2.0f, 2.0f, 2.0f)
		[
			SAssignNew(WrapBox, SWrapBox)
			.UseAllottedSize(true)
		]
	];

	PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this]()
	{
		RequestRebuild();
	}));

	RequestRebuild();
}

void FUniversalObjectLocatorCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (TSharedPtr<IDetailsView> DetailsView = StructBuilder.GetParentCategory().GetParentLayout().GetDetailsViewSharedPtr())
	{
		DetailsView->OnFinishedChangingProperties().AddSPLambda(this, [this](const FPropertyChangedEvent&)
			{
				RequestRebuild();
			});
	}
}

void FUniversalObjectLocatorCustomization::RequestRebuild()
{
	if(!bRebuildRequested)
	{
		bRebuildRequested = true;

		RootWidget->RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSPLambda(this, [this](double, float)
		{
			Rebuild();
			bRebuildRequested = false;
			return EActiveTimerReturnType::Stop;
		}));
	}
}

void FUniversalObjectLocatorCustomization::Rebuild()
{
	Fragments.Reset();

	const FUniversalObjectLocator* CommonValue = GetCommonPropertyValue();
	if(CommonValue)
	{
		// Keep track of current context & class and try to resolve as we traverse the path
		UClass* CurrentContextClass = WeakContextClass.Get(); 
		for(int32 FragmentIndex = 0; FragmentIndex < CommonValue->Fragments.Num(); ++FragmentIndex)
		{
			const FUniversalObjectLocatorFragment& Fragment = CommonValue->Fragments[FragmentIndex];
			TSharedRef<FFragmentItem> NewItem = MakeShared<FFragmentItem>();
			NewItem->FragmentIndex = FragmentIndex;
			NewItem->FragmentType = Fragment.GetFragmentTypeHandle();
			NewItem->LocatorEditorType = Fragment.GetFragmentType() ? Fragment.GetFragmentType()->PrimaryEditorType : NAME_None;
			NewItem->LocatorEditor = ApplicableLocators.FindRef(NewItem->LocatorEditorType);
			NewItem->PropertyHandle = PropertyHandle;
			NewItem->WeakCustomization = SharedThis(this);
			NewItem->bIsTail = FragmentIndex == CommonValue->Fragments.Num() - 1;
			
			NewItem->WeakContextClass = CurrentContextClass;
			if(NewItem->LocatorEditor.IsValid())
			{
				// Make a partial Uol to try to resolve a valid class at this fragment
				FUniversalObjectLocator PartialUol;
				for(int32 PartialFragmentIndex = 0; PartialFragmentIndex < FragmentIndex; ++PartialFragmentIndex)
				{
					PartialUol.Fragments.Add(CommonValue->Fragments[PartialFragmentIndex]);
				}

				FResolveResult Result = PartialUol.Resolve(FResolveParams(GetContext()));
				CurrentContextClass = NewItem->LocatorEditor->ResolveClass(Fragment, Result.SyncGet().Object);
				NewItem->WeakResolvedClass = CurrentContextClass;
			}
			else
			{
				CurrentContextClass = nullptr;
			}

			Fragments.Add(NewItem);
		}
	}

	// Clear the cache to force cache to rebuild
	CachedData.PropertyValue.Reset();

	WrapBox->ClearChildren();

	for(TSharedRef<FFragmentItem> FragmentItem : Fragments)
	{
		TWeakPtr<FFragmentItem> WeakFragmentItem = FragmentItem;

		WrapBox->AddSlot()
		.Padding(0.0f, 0.0f, 1.0f, 0.0f)
		[
			SNew(SDropTarget)
			.OnAllowDrop(this, &FUniversalObjectLocatorCustomization::HandleIsDragAllowed, WeakFragmentItem)
			.OnDropped(this, &FUniversalObjectLocatorCustomization::HandleDrop, WeakFragmentItem)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SComboButton)
					.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
					.ToolTipText(LOCTEXT("SeperatorTooltip", "Edit this fragment"))
					.HasDownArrow(false)
					.OnGetMenuContent(this, &FUniversalObjectLocatorCustomization::GetUserExposedFragmentTypeList, WeakFragmentItem)
					.ButtonContent()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.ChevronRight"))
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SComboButton)
					.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
					.ToolTipText(this, &FUniversalObjectLocatorCustomization::GetFragmentTooltipText, WeakFragmentItem)
					.HasDownArrow(false)
					.OnGetMenuContent(this, &FUniversalObjectLocatorCustomization::GetFragmentTypeWidget, WeakFragmentItem)
					.ButtonContent()
					[
						SNew(SBox)
						.MinDesiredHeight(16.0f)
						.VAlign(VAlign_Center)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SBox)
								.MaxDesiredWidth(16.0f)
								.MaxDesiredHeight(16.0f)
								[
									SNew(SImage)
									.Image(this, &FUniversalObjectLocatorCustomization::GetFragmentIcon, WeakFragmentItem)
								]
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(2.0f, 0.0f, 0.0f, 0.0f)
							[
								SNew(STextBlock)
								.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
								.Text(this, &FUniversalObjectLocatorCustomization::GetFragmentText, WeakFragmentItem)
							]
						]
					]
				]
			]
		];
	}

	WrapBox->AddSlot()
	[
		SNew(SDropTarget)
		.OnAllowDrop(this, &FUniversalObjectLocatorCustomization::HandleIsDragAllowed, TWeakPtr<FFragmentItem>())
		.OnDropped(this, &FUniversalObjectLocatorCustomization::HandleDrop, TWeakPtr<FFragmentItem>())
		[
			SNew(SComboButton)
			.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
			.ToolTipText(LOCTEXT("SeperatorAddFragmentTooltip", "Add a fragment to this locator"))
			.HasDownArrow(false)
			.OnGetMenuContent(this, &FUniversalObjectLocatorCustomization::GetUserExposedFragmentTypeList, TWeakPtr<FFragmentItem>())
			.ButtonContent()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.ChevronRight"))
			]
		]
	];

	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	if(CommonValue == nullptr && Objects.Num() > 1)
	{
		WrapBox->AddSlot()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MultipleValues", "Multiple Values"))
			.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
		];
	}
}

void FUniversalObjectLocatorCustomization::TrimAbsoluteFragments()
{
	// When edited, certain configurations are invalid (like absolute fragments in the middle of a locator's path),
	// so we collapse the edited fragments here
	PropertyHandle->EnumerateRawData([this](void* RawData, int32 Index, int32 Num){
		FUniversalObjectLocator* Reference = static_cast<FUniversalObjectLocator*>(RawData);

		for(int32 FragmentIndex = 0; FragmentIndex < Reference->Fragments.Num(); ++FragmentIndex)
		{
			const FUniversalObjectLocatorFragment& Fragment = Reference->Fragments[FragmentIndex];
			if(TSharedPtr<ILocatorFragmentEditor> LocatorEditor = ApplicableLocators.FindRef(Fragment.GetFragmentType()->PrimaryEditorType))
			{
				if(LocatorEditor->GetLocatorFragmentEditorType() == ELocatorFragmentEditorType::Absolute)
				{
					if(FragmentIndex != 0)
					{
						// Absolute fragments always trim fragments before them
						Reference->Fragments.RemoveAt(0, FragmentIndex);
						break;
					}
				}
			}
		}

		return true;
	});
}

UObject* FUniversalObjectLocatorCustomization::GetSingleObject() const
{
	return GetCachedData().WeakObject.Get();
}

void FUniversalObjectLocatorCustomization::SetValue(FUniversalObjectLocator&& InNewValue)
{
	FScopedTransaction Transaction(LOCTEXT("EditLocatorTransaction", "Edit Locator"));

	PropertyHandle->NotifyPreChange();

	PropertyHandle->EnumerateRawData([&InNewValue](void* RawData, int32 Index, int32 Num){
		FUniversalObjectLocator* Reference = static_cast<FUniversalObjectLocator*>(RawData);
		if (Index == Num-1)
		{
			*Reference = MoveTemp(InNewValue);
		}
		else
		{
			*Reference = InNewValue;
		}
		return true;
	});

	TrimAbsoluteFragments();

	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
}

FString FUniversalObjectLocatorCustomization::GetPathToObject() const
{
	return GetCachedData().ObjectPath;
}

TSharedPtr<IPropertyHandle> FUniversalObjectLocatorCustomization::GetProperty() const
{
	return PropertyHandle;
}

TSharedRef<SWidget> FUniversalObjectLocatorCustomization::GetUserExposedFragmentTypeList(TWeakPtr<FFragmentItem> InWeakFragmentItem)
{
	struct FMenuData
	{
		TSharedPtr<ILocatorFragmentEditor> LocatorEditor;
		FText DisplayText;
		FText DisplayTooltip;
		FSlateIcon DisplayIcon;
		FName LocatorEditorType;
	};

	TArray<FMenuData> MenuData;

	for (TPair<FName, TSharedPtr<ILocatorFragmentEditor>> Pair : ApplicableLocators)
	{
		MenuData.Add(FMenuData{
			Pair.Value,
			Pair.Value->GetDisplayText(),
			Pair.Value->GetDisplayTooltip(),
			Pair.Value->GetDisplayIcon(),
			Pair.Key
		});
	}

	if (MenuData.Num() == 0)
	{
		// 
		return SNullWidget::NullWidget;
	}

	Algo::SortBy(MenuData, &FMenuData::DisplayText, FText::FSortPredicate(ETextComparisonLevel::Default));

	const bool bCloseAfterSelection = true;
	const bool bCloseSelfOnly = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr, nullptr, bCloseSelfOnly);

	TSharedPtr<FFragmentItem> FragmentItem = InWeakFragmentItem.Pin();
	if(FragmentItem.IsValid() && FragmentItem->FragmentIndex == 0)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ClearFragmentsLabel", "Clear Fragments"),
			LOCTEXT("ClearFragmentsTooltip", "Remove all locator fragments"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FUniversalObjectLocatorCustomization::ClearFragments)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}

	const bool bIsTail = !FragmentItem.IsValid();
	if(!bIsTail)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("RemoveFragmentLabel", "Remove Fragment"),
			LOCTEXT("RemoveFragmentTooltip", "Remove this locator fragment"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FUniversalObjectLocatorCustomization::RemoveFragment, InWeakFragmentItem)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}

	FCanExecuteAction AlwaysExecute = FCanExecuteAction::CreateLambda([]{ return true; });

	const FText SectionLabel = bIsTail ?
		LOCTEXT("AddFragmentTypeHeader", "Add a new locator fragment") :
		LOCTEXT("ChangeFragmentTypeHeader", "Change this locator fragment type");
	MenuBuilder.BeginSection(NAME_None, SectionLabel);
	for (const FMenuData& Item : MenuData)
	{
		// Skip absolute fragments after the first fragment
		const bool bIsAfterFirstFragment = (FragmentItem.IsValid() && FragmentItem->FragmentIndex != 0) || (bIsTail && Fragments.Num() != 0); 
		if(bIsAfterFirstFragment && Item.LocatorEditor->GetLocatorFragmentEditorType() == ELocatorFragmentEditorType::Absolute)
		{
			continue;
		}
		
		MenuBuilder.AddMenuEntry(
			Item.DisplayText,
			Item.DisplayTooltip,
			Item.DisplayIcon,
			FUIAction(
				FExecuteAction::CreateSP(this, &FUniversalObjectLocatorCustomization::ChangeEditorType, TWeakPtr<ILocatorFragmentEditor>(Item.LocatorEditor), InWeakFragmentItem),
				AlwaysExecute,
				FIsActionChecked::CreateSP(this, &FUniversalObjectLocatorCustomization::CompareCurrentEditorType, TWeakPtr<ILocatorFragmentEditor>(Item.LocatorEditor), InWeakFragmentItem)
			),
			NAME_None,
			bIsTail ? EUserInterfaceActionType::Button : EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FUniversalObjectLocatorCustomization::ChangeEditorType(TWeakPtr<ILocatorFragmentEditor> InNewLocatorEditor, TWeakPtr<FFragmentItem> InWeakFragmentItem)
{
	TSharedPtr<FFragmentItem> FragmentItem = InWeakFragmentItem.Pin();

	const FText TransactionText = !FragmentItem.IsValid() ?
		LOCTEXT("AddFragmentTransaction", "Add Locator Fragment") :
		LOCTEXT("ChangeLocatorFragmentTypeTransaction", "Change Locator Fragment Type");
	FScopedTransaction Transaction(TransactionText);

	PropertyHandle->NotifyPreChange();

	PropertyHandle->EnumerateRawData(
		[InNewLocatorEditor, this, &FragmentItem](void* Data, int32 DataIndex, int32 Num)
		{
			if (TSharedPtr<ILocatorFragmentEditor> LocatorEditor = InNewLocatorEditor.Pin())
			{
				FUniversalObjectLocator* Ref = static_cast<FUniversalObjectLocator*>(Data);
				if(FragmentItem.IsValid() && Ref->Fragments.IsValidIndex(FragmentItem->FragmentIndex))
				{
					// Trim the remaining fragments beyond this one
					Ref->Fragments.RemoveAt(FragmentItem->FragmentIndex, Ref->Fragments.Num() - FragmentItem->FragmentIndex);
					Ref->AddFragment(LocatorEditor->MakeDefaultLocatorFragment());
				}
				else
				{
					Ref->AddFragment(LocatorEditor->MakeDefaultLocatorFragment());
				}
			}

			return true;
		}
	);

	TrimAbsoluteFragments();

	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
}

void FUniversalObjectLocatorCustomization::RemoveFragment(TWeakPtr<FFragmentItem> InWeakFragmentItem)
{
	TSharedPtr<FFragmentItem> FragmentItem = InWeakFragmentItem.Pin();
	if(!FragmentItem.IsValid())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveLocatorFragmentTransaction", "Remove Locator Fragment"));
	
	PropertyHandle->NotifyPreChange();

	PropertyHandle->EnumerateRawData(
		[this, &FragmentItem](void* Data, int32 DataIndex, int32 Num)
		{
			FUniversalObjectLocator* Ref = static_cast<FUniversalObjectLocator*>(Data);
			if(FragmentItem.IsValid() && Ref->Fragments.IsValidIndex(FragmentItem->FragmentIndex))
			{
				Ref->Fragments.RemoveAt(FragmentItem->FragmentIndex);
			}
			return true;
		}
	);

	TrimAbsoluteFragments();

	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
}

void FUniversalObjectLocatorCustomization::ClearFragments()
{
	FScopedTransaction Transaction(LOCTEXT("ClearLocatorFragmentsTransaction", "Clear Locator Fragments"));
	
	PropertyHandle->NotifyPreChange();

	PropertyHandle->EnumerateRawData(
		[this](void* Data, int32 DataIndex, int32 Num)
		{
			FUniversalObjectLocator* Ref = static_cast<FUniversalObjectLocator*>(Data);
			Ref->Reset();
			return true;
		}
	);

	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
}

bool FUniversalObjectLocatorCustomization::CompareCurrentEditorType(TWeakPtr<ILocatorFragmentEditor> InNewLocatorEditor, TWeakPtr<FFragmentItem> InWeakFragmentItem) const
{
	TSharedPtr<FFragmentItem> FragmentItem = InWeakFragmentItem.Pin();
	if(!FragmentItem.IsValid())
	{
		return false;
	}
	
	return FragmentItem->LocatorEditor == InNewLocatorEditor.Pin();
}

FText FUniversalObjectLocatorCustomization::GetFragmentText(TWeakPtr<FFragmentItem> InWeakFragmentItem) const
{
	const FText NoneText = LOCTEXT("NoValues", "None");
	TSharedPtr<FFragmentItem> FragmentItem = InWeakFragmentItem.Pin();
	if(!FragmentItem.IsValid())
	{
		return NoneText;
	}

	if(FragmentItem->DisplayText.IsSet())
	{
		return FragmentItem->DisplayText.GetValue();
	}
	
	if (FragmentItem->LocatorEditor.IsValid())
	{
		FragmentItem->DisplayText = FragmentItem->LocatorEditor->GetDisplayText(&FragmentItem->GetFragment());
		return FragmentItem->DisplayText.GetValue();
	}

	if(FragmentItem->LocatorEditorType != NAME_None)
	{
		FragmentItem->DisplayText = FText::FromName(FragmentItem->LocatorEditorType);
		return FragmentItem->DisplayText.GetValue();
	}

	return NoneText;
}

FText FUniversalObjectLocatorCustomization::GetFragmentTooltipText(TWeakPtr<FFragmentItem> InWeakFragmentItem) const
{
	// If a rebuild has been requested, return a blank item, as we need to rebuild before we grab the tooltip text,  otherwise we might be dealing with stale or garbage data
	const FText NoneText = LOCTEXT("NoValues", "None");
	TSharedPtr<FFragmentItem> FragmentItem = InWeakFragmentItem.Pin();
	if(!FragmentItem.IsValid() || bRebuildRequested)
	{
		return NoneText;
	}

	if(FragmentItem->TooltipText.IsSet())
	{
		return FragmentItem->TooltipText.GetValue();
	}

	if (FragmentItem->LocatorEditor.IsValid())
	{
		FragmentItem->TooltipText = FragmentItem->LocatorEditor->GetDisplayTooltip(&FragmentItem->GetFragment());
		return FragmentItem->TooltipText.GetValue();
	}

	if (FragmentItem->LocatorEditorType != NAME_None)
	{
		FragmentItem->TooltipText = FText::FromName(FragmentItem->LocatorEditorType);
		return FragmentItem->TooltipText.GetValue();
	}

	return NoneText;
}

const FSlateBrush* FUniversalObjectLocatorCustomization::GetFragmentIcon(TWeakPtr<FFragmentItem> InWeakFragmentItem) const
{
	TSharedPtr<FFragmentItem> FragmentItem = InWeakFragmentItem.Pin();
	if(!FragmentItem.IsValid())
	{
		return nullptr;
	}

	if(FragmentItem->Icon.IsSet())
	{
		return FragmentItem->Icon.GetValue();
	}
	
	if (FragmentItem->LocatorEditorType != NAME_None)
	{
		if(const TSharedPtr<ILocatorFragmentEditor>* FoundLocator = ApplicableLocators.Find(FragmentItem->LocatorEditorType))
		{
			check(FoundLocator->IsValid());
			FragmentItem->Icon = FoundLocator->Get()->GetDisplayIcon(&FragmentItem->GetFragment()).GetIcon();
			return FragmentItem->Icon.GetValue();
		}
	}

	return nullptr;
}

TSharedRef<SWidget> FUniversalObjectLocatorCustomization::GetFragmentTypeWidget(TWeakPtr<FFragmentItem> InWeakFragmentItem)
{
	TSharedPtr<FFragmentItem> FragmentItem = InWeakFragmentItem.Pin();
	if(!FragmentItem.IsValid())
	{
		return SNullWidget::NullWidget;
	}
	
	FUniversalObjectLocatorEditorModule& Module = FModuleManager::Get().LoadModuleChecked<FUniversalObjectLocatorEditorModule>("UniversalObjectLocatorEditor");
	TSharedPtr<ILocatorFragmentEditor> LocatorEditor = ApplicableLocators.FindRef(FragmentItem->LocatorEditorType);
	if (LocatorEditor)
	{
		TSharedPtr<SWidget> EditUI = LocatorEditor->MakeEditUI({ SharedThis(const_cast<FUniversalObjectLocatorCustomization*>(this)), FragmentItem.ToSharedRef() });
		if (EditUI)
		{
			return
				SNew(SBox)
				.Padding(5.0f)
				[
					EditUI.ToSharedRef()
				];
		}
	}

	return SNullWidget::NullWidget;
}

bool FUniversalObjectLocatorCustomization::HandleIsDragAllowed(TSharedPtr<FDragDropOperation> InDragOperation, TWeakPtr<FFragmentItem> InWeakFragmentItem)
{
	if(TSharedPtr<FFragmentItem> FragmentItem = InWeakFragmentItem.Pin())
	{
		// Dragging onto an existing fragment
		if(FragmentItem->LocatorEditor.IsValid() && FragmentItem->LocatorEditor->IsDragSupported(InDragOperation, nullptr))
		{
			return true;
		}
	}
	else
	{
		// Dragging onto the tail
		for (TPair<FName, TSharedPtr<ILocatorFragmentEditor>> Pair : ApplicableLocators)
		{
			if (Pair.Value->IsDragSupported(InDragOperation, nullptr))
			{
				return true;
			}
		}
	}
	
	return false;
}

FReply FUniversalObjectLocatorCustomization::HandleDrop(const FGeometry& Geometry, const FDragDropEvent& DragDropEvent, TWeakPtr<FFragmentItem> InWeakFragmentItem)
{
	TSharedPtr<FDragDropOperation> DragOperation = DragDropEvent.GetOperation();
	
	if(TSharedPtr<FFragmentItem> FragmentItem = InWeakFragmentItem.Pin())
	{
		if(!FragmentItem->LocatorEditor.IsValid())
		{
			return FReply::Unhandled();
		}

		// Dropping onto an existing fragment
		UObject* ResolvedObject = FragmentItem->LocatorEditor->ResolveDragOperation(DragOperation, nullptr);
		if (!ResolvedObject)
		{
			return FReply::Unhandled();
		}

		FScopedTransaction Transaction(LOCTEXT("EditLocatorTransaction", "Edit Locator"));

		PropertyHandle->NotifyPreChange();

		TArray<void*> RawData;
		PropertyHandle->AccessRawData(RawData);
		for (void* Ptr : RawData)
		{
			FUniversalObjectLocator* Reference = static_cast<FUniversalObjectLocator*>(Ptr);
			if(Reference->Fragments.IsValidIndex(FragmentItem->FragmentIndex))
			{
				// Remove all fragments beyond this one 
				Reference->Fragments.RemoveAt(FragmentItem->FragmentIndex, Reference->Fragments.Num() - FragmentItem->FragmentIndex);

				// Resolve to get the context for the new fragment
				UObject* Context = nullptr;
				if(Reference->Fragments.Num() > 0)
				{
					FResolveResult Result = Reference->Resolve(GetContext());
					FResolveResultData ResultData = Result.SyncGet();
					Context = ResultData.Object; 
				}
				else
				{
					Context = GetContext();
				}
				Reference->AddFragment(ResolvedObject, Context, nullptr);
			}
		}

		TrimAbsoluteFragments();

		PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyHandle->NotifyFinishedChangingProperties();

		return FReply::Handled();
	}
	else
	{
		// Dropping onto the tail
		UObject* ResolvedObject = nullptr;

		for (TPair<FName, TSharedPtr<ILocatorFragmentEditor>> Pair : ApplicableLocators)
		{
			ResolvedObject = Pair.Value->ResolveDragOperation(DragOperation, nullptr);
			if (ResolvedObject)
			{
				break;
			}
		}

		if (!ResolvedObject)
		{
			return FReply::Unhandled();
		}

		FScopedTransaction Transaction(LOCTEXT("EditLocatorTransaction", "Edit Locator"));

		PropertyHandle->NotifyPreChange();

		TArray<void*> RawData;
		PropertyHandle->AccessRawData(RawData);
		for (void* Ptr : RawData)
		{
			FUniversalObjectLocator* Reference = static_cast<FUniversalObjectLocator*>(Ptr);

			// Resolve to get the context for the new fragment
			UObject* Context = nullptr;
			if(Reference->Fragments.Num() > 0)
			{
				FResolveResult Result = Reference->Resolve(GetContext());
				FResolveResultData ResultData = Result.SyncGet();
				Context = ResultData.Object; 
			}
			else
			{
				Context = GetContext();
			}
			Reference->AddFragment(ResolvedObject, Context, nullptr);
		}

		TrimAbsoluteFragments();

		PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyHandle->NotifyFinishedChangingProperties();

		return FReply::Handled();
	}
}

void FUniversalObjectLocatorCustomization::SetActor(AActor* NewObject)
{
	FUniversalObjectLocator NewRef(NewObject);

	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);
	for (void* Ptr : RawData)
	{
		FUniversalObjectLocator* Reference = static_cast<FUniversalObjectLocator*>(RawData[0]);
		*Reference = NewRef;
	}
}

FUniversalObjectLocator* FUniversalObjectLocatorCustomization::GetCommonPropertyValue()
{
	const FUniversalObjectLocator* ConstValue = static_cast<const FUniversalObjectLocatorCustomization*>(this)->GetCommonPropertyValue();
	return const_cast<FUniversalObjectLocator*>(ConstValue);
}

const FUniversalObjectLocator* FUniversalObjectLocatorCustomization::GetCommonPropertyValue() const
{
	TOptional<const FUniversalObjectLocator*> CommonValue;
	PropertyHandle->EnumerateConstRawData(
		[&CommonValue](const void* Data, int32 DataIndex, int32 Num)
		{
			if(!CommonValue.IsSet() && Data != nullptr)
			{
				CommonValue = static_cast<const FUniversalObjectLocator*>(Data);
			}
			else if(Data == nullptr || *CommonValue.GetValue() != *static_cast<const FUniversalObjectLocator*>(Data))
			{
				CommonValue = nullptr;
				return false;
			}
			return true;
		}
	);

	return CommonValue.Get(nullptr);
}

const FUniversalObjectLocatorCustomization::FCachedData& FUniversalObjectLocatorCustomization::GetCachedData() const
{
	bool bNeedsUpdate = false;
	if (!CachedData.PropertyValue.IsSet())
	{
		if (const FUniversalObjectLocator* CommonValue = GetCommonPropertyValue())
		{
			if (!CachedData.PropertyValue.IsSet() || *CommonValue != CachedData.PropertyValue.GetValue())
			{
				CachedData.PropertyValue = *CommonValue;
				bNeedsUpdate = true;
			}
		}
		else if (CachedData.PropertyValue.IsSet())
		{
			CachedData.PropertyValue.Reset();
			bNeedsUpdate = true;
		}
	}

	if (bNeedsUpdate)
	{
		if (CachedData.PropertyValue.IsSet())
		{
			CachedData.WeakObject = CachedData.PropertyValue->SyncFind();
		}
		else
		{
			CachedData.WeakObject = nullptr;
		}

		if (UObject* Resolved = CachedData.WeakObject.Get())
		{
			CachedData.ObjectPath = FSoftObjectPath(Resolved).ToString();
		}
		else
		{
			CachedData.ObjectPath.Empty();
		}
	}

	return CachedData;
}

} // namespace UE::UniversalObjectLocator

#undef LOCTEXT_NAMESPACE