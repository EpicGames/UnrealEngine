// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Misc/Optional.h"
#include "Types/SlateEnums.h"

#include "UniversalObjectLocator.h"
#include "IUniversalObjectLocatorCustomization.h"
#include "Layout/Visibility.h"

struct FGeometry;
struct FAssetData;

class FReply;
class FDragDropEvent;
class FDetailWidgetRow;
class FDragDropOperation;

class IPropertyUtilities;
class IDetailChildrenBuilder;

class AActor;

class SBox;
class SWidget;
class SWrapBox;

namespace UE::UniversalObjectLocator
{

class ILocatorFragmentEditor;
struct FFragmentItem;

struct FUniversalObjectLocatorCustomization final : public IPropertyTypeCustomization, public IUniversalObjectLocatorCustomization, public FSelfRegisteringEditorUndoClient
{
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	UObject* GetContext() const override { return WeakContext.Get(); }
	UObject* GetSingleObject() const override;
	FString GetPathToObject() const override;
	void SetValue(FUniversalObjectLocator&& InNewValue) override;
	TSharedPtr<IPropertyHandle> GetProperty() const override;

private:

	struct FCachedData
	{
		TOptional<FUniversalObjectLocator> PropertyValue;
		TWeakObjectPtr<> WeakObject;
		FString ObjectPath;
	};

private:

	virtual void PostUndo(bool bSuccess) override { Rebuild(); }
	virtual void PostRedo(bool bSuccess) override { Rebuild(); }

	void RequestRebuild();
	void Rebuild();

	void TrimAbsoluteFragments();

	TSharedRef<SWidget> GetUserExposedFragmentTypeList(TWeakPtr<FFragmentItem> InWeakFragmentItem);
	FText GetFragmentText(TWeakPtr<FFragmentItem> InWeakFragmentItem) const;
	FText GetFragmentTooltipText(TWeakPtr<FFragmentItem> InWeakFragmentItem) const;
	const FSlateBrush* GetFragmentIcon(TWeakPtr<FFragmentItem> InWeakFragmentItem) const;

	TSharedRef<SWidget> GetFragmentTypeWidget(TWeakPtr<FFragmentItem> InWeakFragmentItem);

	bool HandleIsDragAllowed(TSharedPtr<FDragDropOperation> InDragOperation, TWeakPtr<FFragmentItem> InWeakFragmentItem);
	FReply HandleDrop(const FGeometry& InGeometry, const FDragDropEvent& InDropEvent, TWeakPtr<FFragmentItem> InWeakFragmentItem);

	void SetActor(AActor* InActor);

	const FCachedData& GetCachedData() const;

	FUniversalObjectLocator* GetCommonPropertyValue();
	const FUniversalObjectLocator* GetCommonPropertyValue() const;

	void ChangeEditorType(TWeakPtr<ILocatorFragmentEditor> InNewLocatorEditor, TWeakPtr<FFragmentItem> InWeakFragmentItem);
	bool CompareCurrentEditorType(TWeakPtr<ILocatorFragmentEditor> InNewLocatorEditor, TWeakPtr<FFragmentItem> InWeakFragmentItem) const;

	void RemoveFragment(TWeakPtr<FFragmentItem> InWeakFragmentItem);
	void ClearFragments();

private:

	TSharedPtr<IPropertyUtilities> PropertyUtilities;
	TSharedPtr<IPropertyHandle> PropertyHandle;

	mutable FCachedData CachedData;

	TMap<FName, TSharedPtr<ILocatorFragmentEditor>> ApplicableLocators;

	mutable TArray<TSharedRef<FFragmentItem>> Fragments;

	TSharedPtr<SWidget> RootWidget;
	TSharedPtr<SWrapBox> WrapBox;

	TWeakObjectPtr<> WeakContext;
	TWeakObjectPtr<UClass> WeakContextClass;

	bool bRebuildRequested = false;

	friend struct FFragmentItem;
};


} // namespace UE::UniversalObjectLocator