// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocatorEditor.h"
#include "UniversalObjectLocator.h"

namespace UE::UniversalObjectLocator
{
	TSharedPtr<SWidget> ILocatorFragmentEditor::MakeEditUI(const FEditUIParameters& InParameters)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MakeEditUI(InParameters.Customization);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FUniversalObjectLocator ILocatorFragmentEditor::MakeDefaultLocator() const
	{
		return FUniversalObjectLocator();
	}

	UClass* ILocatorFragmentEditor::ResolveClass(const FUniversalObjectLocatorFragment& InFragment, UObject* InContext) const
	{
		const FResolveParams ResolveParams(InContext);
		UObject* Object = InFragment.Resolve(ResolveParams).SyncGet().Object;
		return Object != nullptr ? Object->GetClass() : nullptr;
	}
}