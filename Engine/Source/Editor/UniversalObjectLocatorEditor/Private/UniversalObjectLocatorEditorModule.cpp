// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocatorEditorModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "UniversalObjectLocatorCustomization.h"

#include "AssetLocatorEditor.h"
#include "ActorLocatorEditor.h"
#include "AnimInstanceLocatorEditor.h"

#include "ISequencerModule.h"

#include "MessageLogModule.h"

namespace UE::UniversalObjectLocator
{

void FUniversalObjectLocatorEditorModule::StartupModule()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout("UniversalObjectLocator", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::UniversalObjectLocator::FUniversalObjectLocatorCustomization::MakeInstance));

	// Register UOL editors
	{
		RegisterLocatorEditor("Actor", MakeShared<FActorLocatorEditor>());
		RegisterLocatorEditor("AnimInstance", MakeShared<FAnimInstanceLocatorEditor>());
		RegisterLocatorEditor("Asset", MakeShared<FAssetLocatorEditor>());
	}

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.RegisterLogListing("UOL", NSLOCTEXT("UOL", "UOLMessageLabel", "Universal Object Locator"));
}

void FUniversalObjectLocatorEditorModule::ShutdownModule()
{
}

void FUniversalObjectLocatorEditorModule::RegisterLocatorEditor(FName Name, TSharedPtr<ILocatorFragmentEditor> LocatorEditor)
{
	LocatorEditors.Add(Name, LocatorEditor);
}

void FUniversalObjectLocatorEditorModule::UnregisterLocatorEditor(FName Name)
{
	LocatorEditors.Remove(Name);
}

TSharedPtr<ILocatorFragmentEditor> FUniversalObjectLocatorEditorModule::FindLocatorEditor(FName Name)
{
	return LocatorEditors.FindRef(Name);
}

void FUniversalObjectLocatorEditorModule::RegisterEditorContext(FName Name, TSharedPtr<ILocatorFragmentEditorContext> LocatorEditorContext)
{
	LocatorEditorContexts.Add(Name, LocatorEditorContext);
}

void FUniversalObjectLocatorEditorModule::UnregisterEditorContext(FName Name)
{
	LocatorEditorContexts.Remove(Name);
}

} // namespace UE::UniversalObjectLocator

IMPLEMENT_MODULE(UE::UniversalObjectLocator::FUniversalObjectLocatorEditorModule, UniversalObjectLocatorEditor);

