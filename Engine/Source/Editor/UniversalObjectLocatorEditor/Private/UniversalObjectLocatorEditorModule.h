// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UniversalObjectLocatorFwd.h"
#include "IUniversalObjectLocatorEditorModule.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

namespace UE::UniversalObjectLocator
{

class ILocatorFragmentEditor;

class FUniversalObjectLocatorEditorModule
	: public IUniversalObjectLocatorEditorModule
{
public:

	void StartupModule() override;

	void ShutdownModule() override;

	void RegisterLocatorEditor(FName LocatorName, TSharedPtr<ILocatorFragmentEditor> LocatorEditor) override;

	void UnregisterLocatorEditor(FName LocatorName) override;

	TSharedPtr<ILocatorFragmentEditor> FindLocatorEditor(FName Name) override;

	void RegisterEditorContext(FName Name, TSharedPtr<ILocatorFragmentEditorContext> LocatorEditorContext) override;

	void UnregisterEditorContext(FName Name) override;

	TMap<FName, TSharedPtr<ILocatorFragmentEditor>> LocatorEditors;

	TMap<FName, TSharedPtr<ILocatorFragmentEditorContext>> LocatorEditorContexts;
};

} // namespace UE::UniversalObjectLocator


