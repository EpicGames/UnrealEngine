// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

struct FUniversalObjectLocator;
struct FUniversalObjectLocatorFragment;

class FString;
class UObject;
class IPropertyHandle;

namespace UE::UniversalObjectLocator
{

class IFragmentEditorHandle
{
public:
	virtual ~IFragmentEditorHandle() = default;

	// Get the fragment that this handle refers to
	virtual const FUniversalObjectLocatorFragment& GetFragment() const = 0;

	// Get the context class that this fragment is resolved against
	virtual const UClass* GetContextClass() const = 0;

	// Get the class that this fragment is resolved to
	virtual const UClass* GetResolvedClass() const = 0;

	// Set the value of this fragment
	virtual void SetValue(const FUniversalObjectLocatorFragment& InNewValue) = 0;
};

class IUniversalObjectLocatorCustomization
{
public:
	virtual ~IUniversalObjectLocatorCustomization() = default;

	virtual UObject* GetContext() const = 0;
	virtual UObject* GetSingleObject() const = 0;
	virtual FString GetPathToObject() const = 0;
	virtual void SetValue(FUniversalObjectLocator&& InNewValue) = 0;

	virtual TSharedPtr<IPropertyHandle> GetProperty() const = 0;
};


} // namespace UE::UniversalObjectLocator