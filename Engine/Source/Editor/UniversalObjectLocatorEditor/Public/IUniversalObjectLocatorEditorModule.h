// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointerFwd.h"

class FName;

namespace UE::UniversalObjectLocator
{

class ILocatorFragmentEditor;
class ILocatorFragmentEditorContext;

class IUniversalObjectLocatorEditorModule
	: public IModuleInterface
{
public:
	// Register a locator editor
	// @param Name                     The name of the locator editor
	// @param LocatorEditor            The locator editor to register
	virtual void RegisterLocatorEditor(FName Name, TSharedPtr<ILocatorFragmentEditor> LocatorEditor) = 0;

	// Unregister a locator editor previously registered via RegisterLocatorEditor
	// @param Name                     The name of the locator editor
	virtual void UnregisterLocatorEditor(FName Name) = 0;

	// Find a locator editor previously registered via RegisterLocatorEditor
	// @param Name                     The name of the locator editor
	virtual TSharedPtr<ILocatorFragmentEditor> FindLocatorEditor(FName Name) = 0;

	// Register a locator editor context, used to control editor behavior on a per-use basis
	// @param Name                     The name of the context, used as an identifier
	// @param LocatorEditorContext     The context to register
	virtual void RegisterEditorContext(FName Name, TSharedPtr<ILocatorFragmentEditorContext> LocatorEditorContext) = 0;

	// Unregister a locator editor context previously registered via RegisterEditorContext
	// @param Name             The name of the context, used as an identifier
	virtual void UnregisterEditorContext(FName Name) = 0;
};


} // namespace UE::UniversalObjectLocator