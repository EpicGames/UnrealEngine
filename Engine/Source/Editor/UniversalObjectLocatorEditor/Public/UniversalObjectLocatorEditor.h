// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Internationalization/Text.h"
#include "Textures/SlateIcon.h"

#define UE_API UNIVERSALOBJECTLOCATOREDITOR_API

class UObject;
class SWidget;
class FDragDropOperation;
class IPropertyHandle;
class IDetailChildrenBuilder;
class IPropertyTypeCustomizationUtils;
struct FUniversalObjectLocator;
struct FUniversalObjectLocatorFragment;

namespace UE::UniversalObjectLocator
{

class IFragmentEditorHandle;
class IUniversalObjectLocatorCustomization;

enum class ELocatorFragmentEditorType : uint8
{
	// Locator fragment is relative to something else
	Relative,

	// Locator fragment is absolute
	Absolute,
};

class ILocatorFragmentEditor : public TSharedFromThis<ILocatorFragmentEditor>
{
public:
	virtual ~ILocatorFragmentEditor() = default;

	// Get the type of this locator editor (relative, absolute)
	// @return the locator editor type
	virtual ELocatorFragmentEditorType GetLocatorFragmentEditorType() const = 0;

	// Get whether this locator editor is allowed in the supplied context. @see ILocatorFragmentEditorContext.
	// @param InContextName    The name of the context, if this is NAME_None, no context is supplied (the default for blueprint-instantiated UOL properties)
	// @return true if this locator is allowed in the supplied context
	virtual bool IsAllowedInContext(FName InContextName) const { return true; }
	
	// Called to check whether a drag operation is supported for this fragment
	// @param InDragOperation  The drag operation to check
	// @param InContext        The context object
	// @return true if the operation is supported, false if unsupported
	virtual bool IsDragSupported(TSharedPtr<FDragDropOperation> InDragOperation, UObject* InContext) const = 0;

	// Called to resolve a drag operation to an object
	// @param InDragOperation  The drag operation to resolve
	// @param InContext        The context object
	// @return the resolved object
	virtual UObject* ResolveDragOperation(TSharedPtr<FDragDropOperation> InDragOperation, UObject* InContext) const = 0;

	// Parameters used to create the UI for this locator fragment editor
	struct FEditUIParameters
	{
		// The customization that is creating this UI
		TSharedRef<IUniversalObjectLocatorCustomization> Customization;

		// The handle to the fragment to create the UI for
		TSharedRef<IFragmentEditorHandle> Handle;
	};

	// Make the editor UI for this fragment (displayed in a context menu)
	// @param InParameters     Parameters used to create the UI
	// @return the editing widget
	UE_API virtual TSharedPtr<SWidget> MakeEditUI(const FEditUIParameters& InParameters);

	UE_DEPRECATED(5.5, "Please use MakeEditUI that takes a FEditUIParameters struct")
	virtual TSharedPtr<SWidget> MakeEditUI(TSharedPtr<IUniversalObjectLocatorCustomization> Customization) { return nullptr; }
	
	UE_DEPRECATED(5.5, "This method is mo longer used. Please use MakeEditUI to create your editor display within a popup window per-fragment")
	virtual	void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) {}

	// Get the text to display for the supplied fragment
	// @param InFragment       The fragment to display. If this is nullptr then the name of the fragment type should be returned
	// @return the text to display
	virtual FText GetDisplayText() const { return GetDisplayText(nullptr); }
	virtual FText GetDisplayText(const FUniversalObjectLocatorFragment* InFragment) const = 0;

	// Get the tooltip text to display for this fragment
	// @param InFragment       The fragment to display. If this is nullptr then the tooltip of the fragment type should be returned
	// @return the tooltip text to display
	virtual FText GetDisplayTooltip() const { return GetDisplayTooltip(nullptr); }
	virtual FText GetDisplayTooltip(const FUniversalObjectLocatorFragment* InFragment) const = 0;

	// Get the icon to display for this fragment
	// @param InFragment       The fragment to display. If this is nullptr then the icon of the fragment type should be returned
	// @return the icon to display
	virtual FSlateIcon GetDisplayIcon() const { return GetDisplayIcon(nullptr); }
	virtual FSlateIcon GetDisplayIcon(const FUniversalObjectLocatorFragment* InFragment) const = 0;

	// Resolve the output class of a fragment at edit time
	// @param InFragment       The fragment to resolve the class of
	// @param InContext        The context to resolve the class against
	// @return the class that was resolved from the fragment
	UE_API virtual UClass* ResolveClass(const FUniversalObjectLocatorFragment& InFragment, UObject* InContext) const;

	// Make a default fragment for the fragment type that this editor UI manipulates
	// @return a default fragment for this locator editor to use
	virtual FUniversalObjectLocatorFragment MakeDefaultLocatorFragment() const = 0;

	UE_DEPRECATED(5.5, "No longer required. Please override MakeDefaultLocatorFragment")
	UE_API virtual FUniversalObjectLocator MakeDefaultLocator() const;
};

UE_DEPRECATED(5.5, "Please use ILocatorFragmentEditor")
typedef ILocatorFragmentEditor ILocatorEditor;

} // namespace UE::UniversalObjectLocator

#undef UE_API
