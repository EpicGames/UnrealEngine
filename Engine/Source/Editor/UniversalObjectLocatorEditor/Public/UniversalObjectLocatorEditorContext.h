// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class UObject;
class IPropertyHandle;

namespace UE::UniversalObjectLocator
{

class ILocatorFragmentEditorContext : public TSharedFromThis<ILocatorFragmentEditorContext>
{
public:
	virtual ~ILocatorFragmentEditorContext() = default;

	// Get a context object that can be used to resolve partial fragments at edit time
	virtual UObject* GetContext(const IPropertyHandle& InPropertyHandle) const = 0;

	// Check whether the supplied locator fragment is supported in this context
	virtual bool IsFragmentAllowed(FName InFragmentName) const { return true; }
};

} // namespace UE::UniversalObjectLocator
