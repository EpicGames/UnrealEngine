// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class UniversalObjectLocatorEditor : ModuleRules
{
	public UniversalObjectLocatorEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		// @todo: this is temporarily disabled due to some MovieScene code not compiling
		//CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"PropertyEditor",
				"SlateCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"EditorWidgets",
				"Engine",
				"MessageLog",
				"PropertyEditor",
				"SceneOutliner",
				"Slate",
				"SlateCore",
				"InputCore",
				"Sequencer",
				"UniversalObjectLocator",
				"UnrealEd",
			}
		);
	}
}
